[Audio]
; Defining below allows switching to audio mixer using -audiomixer commandline
AudioCaptureModuleName=AudioCaptureRtAudio
AudioMixerModuleName=AudioMixerCoreAudio
PlatformFormat=OGG
PlatformStreamingFormat=OGG

; Defines a platform-specific volume headroom (in dB) for audio to provide better platform consistency with respect to volume levels.
PlatformHeadroomDB=-6

[DevOptions.Shaders]
MaxShaderJobBatchSize=25

[OnlineSubsystem]
LocalPlatformName=MAC

[PlatformCrypto]
PlatformRequiresDataCrypto=True

[PlatformPaths]
UnrealBuildTool=Engine/Build/BatchFiles/Mac/Build.sh

[TextureStreaming]
; PoolSizeVRAMPercentage is how much percentage of GPU Dedicated VRAM should be used as a TexturePool cache for streaming textures (0 - unlimited streaming)
PoolSizeVRAMPercentage=70

[SystemSettings]
r.setres=1280x720
g.TimeoutForBlockOnRenderFence=300000
framepro.ScopeMinTimeMicroseconds=10
fx.NiagaraAllowRuntimeScalabilityChanges=1
;Perplatform to PerQualityLevel conversion mapping for platform groups
QualityLevelMapping="high"

[ConsoleVariables]
r.HairStrands.Visibility.MSAA.SamplePerPixel=1
r.SceneCulling.ExplicitCellBounds=1
; Workaround for the webkit WebBrowser issues with slate
Slate.NoAnimationOnTabForegroundedEvent=True

[/Script/Engine.RendererSettings]
; Substrate settings for more advanced shading on PC and editor by default.
r.Substrate.RoughDiffuse=1
r.Substrate.ShadingQuality=1
r.Substrate.TileCoord8bits=0
r.Substrate.Glints=1
r.Substrate.ClosuresPerPixel=4
r.Refraction.Blur=1

[ShaderPlatformConfig METAL_ES3_1]
BindlessConfiguration=Disabled

[ShaderPlatformConfig METAL_SM5]
BindlessConfiguration=Disabled

[ShaderPlatformConfig METAL_SM6]
BindlessConfiguration=All

[/Script/GameplayDebugger.GameplayDebuggingControllerComponent]
CategoryZeroBind=(Key=NumPadZero,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryOneBind=(Key=NumPadOne,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryTwoBind=(Key=NumPadTwo,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryThreeBind=(Key=NumPadThree,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryFourBind=(Key=NumPadFour,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryFiveBind=(Key=NumPadFive,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategorySixBind=(Key=NumPadSix,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategorySevenBind=(Key=NumPadSeven,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryEightBind=(Key=NumPadEight,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CategoryNineBind=(Key=NumPadNine,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
CycleDetailsViewBind=(Key=Add,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
DebugCameraBind=(Key=Tab,bShift=False,bCtrl=False,bAlt=False,bCmd=False)
OnScreenDebugMessagesBind=(Key=Tab,bShift=True,bCtrl=False,bAlt=False,bCmd=False)
GameHUDBind=(Key=Tilde,bShift=True,bCtrl=False,bAlt=False,bCmd=False)

[/Script/BuildSettings.BuildSettings]
bCompileCEF3=False
bMultiViewPerSlateWindowSupport=True 
