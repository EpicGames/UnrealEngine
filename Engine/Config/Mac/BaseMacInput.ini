[MacInput]
; This changes what EKeys::Platform_Delete will be set to on the Mac platform.
;
; If this is true (default), then EKeys::Platform_Delete will be EKeys::Backspace.
; If this is false, then EKeys::Platform_Delete will be set to EKeys::Delete.
; 
; See FMacPlatformInput::GetPlatformDeleteKey for implementation
bPlatformDeleteIsBackspace=true

[InputPlatformSettings_Mac InputPlatformSettings]
MaxPlatformUserCount=8
input.DeviceMappingPolicy=1