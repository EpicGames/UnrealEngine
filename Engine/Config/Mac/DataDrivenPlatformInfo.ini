[DataDrivenPlatformInfo]
bIsConfidential=false
TargetSettingsIniSectionName=/Script/MacTargetPlatform.MacTargetSettings
bHasDedicatedGamepad=false
bDefaultInputStandardKeyboard=false
bInputSupportConfigurable=true
DefaultInputType=MouseAndKeyboard
bSupportsMouseAndKeyboard=true
bSupportsGamepad=true
bCanChangeGamepadType=true
bSupportsTouch=false
GlobalIdentifier=003BE29617004F0C8E1F786081EFBB1F

NormalIconPath=Launcher/Mac/Platform_Mac_24x
LargeIconPath=Launcher/Mac/Platform_Mac_128x
XLargeIconPath=
AutoSDKPath=
TutorialPath=SharingAndReleasing
Windows:bIsEnabled=true
Mac:bIsEnabled=true
Linux:bIsEnabled=false
bUsesHostCompiler=true
bUATClosesAfterLaunch=true
PlatformGroupName=Desktop
ICUDataVersion=icudt64l

[ShaderPlatform METAL_SM5]
Language=Metal
MaxFeatureLevel=SM5
ShaderFormat=SF_METAL_SM5
bIsPC=true
bSupportsDebugViewShaders=true
bSupportsAnisotropicMaterials=true
bSupportsDistanceFields=true
bSupportsGen5TemporalAA=true
bSupportsLumenGI=true
bSupportsSSDIndirect=true
bSupportsWaveOperations=Unsupported
MinimumWaveSize=32
MaximumWaveSize=64
bSupportsGPUScene=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true
bSupportsVertexShaderLayer=true
bSupportsDxc=true
bIsSPIRV=true
bSupportsInstancedStereo=false
SupportsMultiViewport=RuntimeGuaranteed
bSupportsPercentageCloserShadows=true
bSupports4ComponentUAVReadWrite=true
bSupportsVolumeTextureAtomics=false
bSupportsDualSourceBlending=true
EnablesHLSL2021ByDefault=1
bSupportsRayTracing = false
bSupportsRayTracingShaders = false
bSupportsInlineRayTracing = false
bSupportsRayTracingIndirectInstanceData = false
// Reenable bSupportsHairStrandGeometry when bindless will be enabled by default
bSupportsHairStrandGeometry = false
bSupportsNanite=false
bSupportsUInt64ImageAtomics=false
bSupportsNNEShaders=false
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsBindless=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_MacSM5", "Mac Metal SM5")

[ShaderPlatform METAL_SM6]
Language=Metal
MaxFeatureLevel=SM6
ShaderFormat=SF_METAL_SM6
bIsPC=true
bSupportsDebugViewShaders=true
bSupportsAnisotropicMaterials=true
bSupportsDistanceFields=true
bSupportsGen5TemporalAA=true
bSupportsLumenGI=true
bSupportsSSDIndirect=true
bSupportsWaveOperations=RuntimeGuaranteed
MinimumWaveSize=32
MaximumWaveSize=64
bSupportsGPUScene=true
bSupportsFFTBloom=true
bSupportsDiaphragmDOF=true
bSupportsVertexShaderLayer=true
bSupportsDxc=true
bIsSPIRV=true
bSupportsInstancedStereo=false
SupportsMultiViewport=RuntimeGuaranteed
bSupportsPercentageCloserShadows=true
bSupports4ComponentUAVReadWrite=true
bSupportsVolumeTextureAtomics=true
bSupportsDualSourceBlending=true
EnablesHLSL2021ByDefault=1
bSupportsRayTracing=true
bSupportsRayTracingShaders=true
bSupportsInlineRayTracing=true
bSupportsRayTracingProceduralPrimitive=true
bSupportsRayTracingIndirectInstanceData=true
StaticShaderBindingLayoutSupport=RayTracingOnly
bSupportsPathTracing=true
bSupportsRayTracingCallableShaders=true
bSupportsHairStrandGeometry=true
bSupportsNanite=true
bSupportsUInt64ImageAtomics=true
bSupportsNNEShaders=true
bSupportsIndependentSamplers=true
bSupportsComputeFramework=true
bSupportsBindless=true
bSupportsMeshShadersTier0=false
bSupportsMeshShadersTier1=false
bSupportsMeshShadersWithClipDistance=false
bSupportsAdaptiveGBuffer=true
MaxMeshShaderThreadGroupSize=128
SupportsBarycentricsSemantic=RuntimeDependent
bSupportsRealTypes=RuntimeGuaranteed
bSupportsSceneDataCompressedTransforms=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_MacSM6", "Mac Metal SM6")

[ShaderPlatform METAL_ES3_1]
Language=Metal
MaxFeatureLevel=ES3_1
ShaderFormat=SF_METAL_ES3_1
bIsPC=true
bIsMobile=true
bSupports4ComponentUAVReadWrite=true
bSupportsManualVertexFetch=false
bSupportsVolumeTextureAtomics=false
bSupportsShaderPipelines = false
bSupportsDualSourceBlending = true
bSupportsIndependentSamplers=true
bSupportsComputeFramework = true
bSupportsBindless=true
FriendlyName=LOCTEXT("FriendlyShaderPlatformName_MacMobile", "Mac Metal Mobile")

[PreviewPlatform METAL_SM5]
PlatformName=Mac
ShaderFormat=SF_METAL_SM5
ShaderPlatform=METAL_SM5
PreviewFeatureLevel=SM6
MenuTooltip=LOCTEXT("PreviewMenuTooltip_METAL_SM5", "Mac using SM5 profile")

[PreviewPlatform METAL_SM6]
PlatformName=Mac
ShaderFormat=SF_METAL_SM6
ShaderPlatform=METAL_SM6
MenuTooltip=LOCTEXT("PreviewMenuTooltip_METAL_SM6", "Mac using SM6 profile")
