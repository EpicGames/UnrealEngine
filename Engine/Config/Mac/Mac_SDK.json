{
	"ParentSDKFile": "../Apple/Apple_SDK.json",

	"EditorDeploymentTarget": "14.00",
	"EditorBuildTarget": "14.00",

	"DeploymentTarget": "14.00",
	"BuildTarget": "14.00",

	"//": "macOS Sonoma 14.0 (Released Sept 26, 2023)",
	"MinSoftwareVersion": "14.00"
}
